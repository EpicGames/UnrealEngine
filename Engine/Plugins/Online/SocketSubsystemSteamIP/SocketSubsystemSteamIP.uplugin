{
	"FileVersion": 3,

	"FriendlyName": "Socket Subsystem Steam (IP)",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Responsible for Steam net connections between users. Does NOT use NAT punchthrough, use the SteamSockets plugin for P2P support",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "SocketSubsystemSteamIP",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Mac"
			],
			"PlatformArchitectureDenyList": [
				"Win64:arm64"
			]
		}
	],

	"Plugins": [
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "SteamShared",
			"Enabled": true
		}
	]
}
