// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemSteam.h"
#include "Misc/ConfigCacheIni.h"
#include "SocketsSteam.h"
#include "SocketSubsystemModule.h"
#include "SteamNetConnection.h"
#include <steam/isteamgameserver.h>
#include <steam/isteamuser.h>

const FName STEAMIP_SUBSYSTEMNAME(TEXT("STEAM"));
FSocketSubsystemSteam* FSocketSubsystemSteam::SocketSingleton = nullptr;


/** 
 * Singleton interface for this subsystem 
 * @return the only instance of this subsystem
 */
FSocketSubsystemSteam* FSocketSubsystemSteam::Create()
{
	if (SocketSingleton == nullptr)
	{
		SocketSingleton = new FSocketSubsystemSteam();
	}

	return SocketSingleton;
}

/**
 * Performs Steam specific socket clean up
 */
void FSocketSubsystemSteam::Destroy()
{
	if (SocketSingleton != nullptr)
	{
		SocketSingleton->Shutdown();
		delete SocketSingleton;
		SocketSingleton = nullptr;
	}
}

/**
 * Does Steam platform initialization of the sockets library
 *
 * @param Error a string that is filled with error information
 *
 * @return true if initialized ok, false otherwise
 */
bool FSocketSubsystemSteam::Init(FString& Error)
{
	if (GConfig)
	{
		if (!GConfig->GetBool(TEXT("SocketSubsystemSteamIP"), TEXT("bAllowP2PPacketRelay"), bAllowP2PPacketRelay, GEngineIni))
		{
			if (GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bAllowP2PPacketRelay"), bAllowP2PPacketRelay, GEngineIni))
			{
				UE_LOGF(LogSockets, Warning, "bAllowP2PPacketRelay has moved from OnlineSubsystemSteam to SocketSubsystemSteamIP, previous location is depreceated");
			}
			else
			{
				UE_LOGF(LogSockets, Warning, "Missing bAllowP2PPacketRelay key in SocketSubsystemSteamIP of DefaultEngine.ini");
			}
		}

		if (!GConfig->GetFloat(TEXT("SocketSubsystemSteamIP"), TEXT("P2PConnectionTimeout"), P2PConnectionTimeout, GEngineIni))
		{
			if (GConfig->GetFloat(TEXT("OnlineSubsystemSteam"), TEXT("P2PConnectionTimeout"), P2PConnectionTimeout, GEngineIni))
			{
				UE_LOGF(LogSockets, Warning, "P2PConnectionTimeout has moved from OnlineSubsystemSteam to SocketSubsystemSteamIP, previous location is depreceated");
			}
			else
			{
				UE_LOGF(LogSockets, Warning, "Missing P2PConnectionTimeout key in SocketSubsystemSteamIP of DefaultEngine.ini");
			}
		}

		if (!GConfig->GetDouble(TEXT("SocketSubsystemSteamIP"), TEXT("P2PCleanupTimeout"), P2PCleanupTimeout, GEngineIni))
		{
			if (GConfig->GetDouble(TEXT("OnlineSubsystemSteam"), TEXT("P2PCleanupTimeout"), P2PCleanupTimeout, GEngineIni))
			{
				UE_LOGF(LogSockets, Warning, "P2PCleanupTimeout has moved from OnlineSubsystemSteam to SocketSubsystemSteamIP, previous location is depreceated");
			}
			else
			{
				UE_LOGF(LogSockets, Warning, "Missing P2PCleanupTimeout key in SocketSubsystemSteamIP of DefaultEngine.ini");
			}
		}
	}

	if (SteamNetworking())
	{
		SteamNetworking()->AllowP2PPacketRelay(bAllowP2PPacketRelay);
	}

	if (SteamGameServerNetworking())
	{
		SteamGameServerNetworking()->AllowP2PPacketRelay(bAllowP2PPacketRelay);
	}

	return true;
}

/**
 * Performs platform specific socket clean up
 */
void FSocketSubsystemSteam::Shutdown()
{
	for (int32 ConnIdx=SteamConnections.Num()-1; ConnIdx>=0; ConnIdx--)
	{
		if (SteamConnections[ConnIdx].IsValid())
		{
			USteamNetConnection* SteamConn = CastChecked<USteamNetConnection>(SteamConnections[ConnIdx].Get());
			UnregisterConnection(SteamConn);
		}
	}

	UE_LOGF(LogSockets, Verbose, "Shutting down SteamNet connections");
	
	// Empty the DeadConnections list as we're shutting down anyways
	// This is so we don't spend time checking the DeadConnections
	// for duplicate pending closures
	DeadConnections.Empty();

	// Cleanup any remaining sessions
	for (auto SessionIds : AcceptedConnections)
	{
		P2PRemove(CSteamID(SessionIds.Key), -1);
	}

	CleanupDeadConnections(true);

	// Cleanup sockets
	TArray<FSocketSteam*> TempArray = SteamSockets;
	for (int SocketIdx=0; SocketIdx < TempArray.Num(); SocketIdx++)
	{
		DestroySocket(TempArray[SocketIdx]);
	}

	SteamSockets.Empty();
	SteamConnections.Empty();
	AcceptedConnections.Empty();
	DeadConnections.Empty();
}

/**
 * Creates a socket
 *
 * @Param SocketType type of socket to create (DGram, Stream, etc)
 * @param SocketDescription debug description
 * @param ProtocolType the socket protocol to be used
 *
 * @return the new socket or NULL if failed
 */
FSocket* FSocketSubsystemSteam::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	FSocket* NewSocket = nullptr;
	if (SocketType == FName("SteamClientSocket"))
	{
		ISteamUser* SteamUserPtr = SteamUser();
		if (SteamUserPtr != nullptr)
		{
			NewSocket = new FSocketSteam(SteamNetworking(), SteamUserPtr->GetSteamID(), SocketDescription, FNetworkProtocolTypes::SteamSocketsIP);

			if (NewSocket)
			{
				AddSocket((FSocketSteam*)NewSocket);
			}
		}
	}
	else if (SocketType == FName("SteamServerSocket"))
	{
		NewSocket = new FSocketSteam(SteamGameServerNetworking(), GameServerCSID, SocketDescription, FNetworkProtocolTypes::SteamSocketsIP);

		if (NewSocket)
		{
			AddSocket((FSocketSteam*)NewSocket);
		}
	}
	else
	{
		ISocketSubsystem* PlatformSocketSub = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		if (PlatformSocketSub)
		{
			NewSocket = PlatformSocketSub->CreateSocket(SocketType, SocketDescription, ProtocolType);
		}
	}

	if (!NewSocket)
	{
		UE_LOGF(LogSockets, Warning, "Failed to create socket %ls [%ls]", *SocketType.ToString(), *SocketDescription);
	}

	return NewSocket;
}

/**
 * Cleans up a socket class
 *
 * @param Socket the socket object to destroy
 */
void FSocketSubsystemSteam::DestroySocket(FSocket* Socket)
{
	// Possible non steam socket here PLATFORM_SOCKETSUBSYSTEM, but its just a pointer compare
	RemoveSocket((FSocketSteam*)Socket);
	delete Socket;
}

/**
 * Associate the game server steam id with any sockets that were created prior to successful login
 *
 * @param GameServerId id assigned to this game server
 */
void FSocketSubsystemSteam::FixupSockets(const CSteamID& GameServerId)
{
	
	for (int32 SockIdx = 0; SockIdx < SteamSockets.Num(); SockIdx++)
	{
		FSocketSteam* Socket = SteamSockets[SockIdx];
		if (Socket->SteamNetworkingPtr == SteamGameServerNetworking() && Socket->LocalSteamId == CSteamID())
		{
			Socket->LocalSteamId = GameServerId;
		}
	}
}

/**
 * Adds a steam connection for tracking
 *
 * @param Connection The connection to add for tracking
 */
void FSocketSubsystemSteam::RegisterConnection(USteamNetConnection* Connection)
{
	check(!Connection->bIsPassthrough);

	FWeakObjectPtr ObjectPtr = Connection;
	SteamConnections.Add(ObjectPtr);

	if (FSocket* CurSocket = Connection->GetSocket())
	{
		TSharedPtr<const FInternetAddr> CurRemoteAddr = Connection->GetRemoteAddr();

		if (CurRemoteAddr.IsValid())
		{
			FSocketSteam* SteamSocket = (FSocketSteam*)CurSocket;
			TSharedPtr<const FInternetAddrSteam> SteamAddr = StaticCastSharedPtr<const FInternetAddrSteam>(CurRemoteAddr);

			UE_LOGF(LogSockets, Log, "Adding user %ls from RegisterConnection", *SteamAddr->ToString(true));

			P2PTouch(SteamSocket->SteamNetworkingPtr, SteamAddr->GetSteamID(), SteamAddr->GetPort());
		}
	}
}

/**
 * Removes a steam connection from tracking
 *
 * @param Connection The connection to remove from tracking
 */
void FSocketSubsystemSteam::UnregisterConnection(USteamNetConnection* Connection)
{
	check(!Connection->bIsPassthrough);

	FWeakObjectPtr ObjectPtr = Connection;

	// Don't call P2PRemove again if we didn't actually remove a connection. This 
	// will get called twice - once the connection is closed and when the connection
	// is garbage collected. It's possible that the player who left rejoined before garbage
	// collection runs (their connection object will be different), so P2PRemove would kick
	// them from the session when it shouldn't.
	if (SteamConnections.RemoveSingleSwap(ObjectPtr) == 1 && Connection->GetRemoteAddr().IsValid())
	{
		TSharedPtr<const FInternetAddrSteam> SteamAddr = StaticCastSharedPtr<const FInternetAddrSteam>(Connection->GetRemoteAddr());
		P2PRemove(SteamAddr->GetSteamID(), SteamAddr->GetPort());
	}
}

void FSocketSubsystemSteam::ConnectFailure(const CSteamID& RemoteId)
{
	// Remove any GC'd references
	for (int32 ConnIdx=SteamConnections.Num()-1; ConnIdx>=0; ConnIdx--)
	{
		if (!SteamConnections[ConnIdx].IsValid())
		{
			SteamConnections.RemoveAt(ConnIdx);
		}
	}

	// Find the relevant connections and shut them down
	for (int32 ConnIdx=0; ConnIdx<SteamConnections.Num(); ConnIdx++)
	{
		USteamNetConnection* SteamConn = CastChecked<USteamNetConnection>(SteamConnections[ConnIdx].Get());
		if (SteamConn)
		{
			TSharedPtr<const FInternetAddrSteam> RemoteAddrSteam = StaticCastSharedPtr<const FInternetAddrSteam>(SteamConn->GetRemoteAddr());
			// Only checking Id here because its a complete failure (channel doesn't matter)
			if (RemoteAddrSteam->GetSteamID() == RemoteId)
			{
				SteamConn->Close();
			}
		}
	}

	P2PRemove(RemoteId, -1);
}

/**
 * Gets the address information of the given hostname and outputs it into an array of resolvable addresses.
 * It is up to the caller to determine which one is valid for their environment.
 *
 * @param HostName string version of the queryable hostname or ip address
 * @param ServiceName string version of a service name ("http") or a port number ("80")
 * @param QueryFlags What flags are used in making the getaddrinfo call. Several flags can be used at once by ORing the values together.
 *                   Platforms are required to translate this value into a the correct flag representation.
 * @param ProtocolType this is used to limit results from the call. Specifying None will search all valid protocols.
 *					   Callers will find they rarely have to specify this flag.
 * @param SocketType What socket type should the results be formatted for. This typically does not change any formatting results and can
 *                   be safely left to the default value.
 *
 * @return the array of results from GetAddrInfo
 */
FAddressInfoResult FSocketSubsystemSteam::GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName,
	EAddressInfoFlags QueryFlags, const FName ProtocolTypeName, ESocketType SocketType)
{
	FString RawAddress(HostName);

	// Remove steam prefixes if they exist
	RawAddress.RemoveFromStart(STEAM_URL_PREFIX);

	// Steam ids are pure numeric values, so we can use this to determine if the input is a SteamID.
	if (RawAddress.IsNumeric() && HostName != nullptr)
	{
		FAddressInfoResult SteamResult(HostName, ServiceName);

		// This is an Steam address
		uint64 Id = FCString::Atoi64(*RawAddress);
		if (Id != 0)
		{
			FString PortString(ServiceName);
			SteamResult.ReturnCode = SE_NO_ERROR;
			TSharedRef<FInternetAddrSteam> SteamIdAddress = StaticCastSharedRef<FInternetAddrSteam>(CreateInternetAddr());
			SteamIdAddress->SetSteamID(CSteamID(Id));
			if (PortString.IsNumeric())
			{
				SteamIdAddress->SetPort(FCString::Atoi(*PortString));
			}

			SteamResult.Results.Add(FAddressInfoResultData(SteamIdAddress, 0, FNetworkProtocolTypes::SteamSocketsIP, SOCKTYPE_Unknown));
			return SteamResult;
		}
		else
		{
			UE_LOGF(LogSockets, Warning, "GetAddressInfo: Could not serialize %ls into a SteamID, the ID was invalid.", *RawAddress);
			SteamResult.ReturnCode = SE_HOST_NOT_FOUND;
			return SteamResult;
		}
	}

	return ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetAddressInfo(HostName, ServiceName, QueryFlags, ProtocolTypeName, SocketType);
}

/**
 * Serializes a string that only contains an address.
 *
 * On Steam, this will take SteamIDs and serialize them into FInternetAddrSteam if it is determined
 * the input string is an ID. Otherwise, this will give you back a FInternetAddrBSD.
 *
 * This is a what you see is what you get, there is no DNS resolution of the input string,
 * so only use this if you know you already have a valid ip address.
 * Otherwise, feed the address to GetAddressInfo for guaranteed results.
 *
 * @param InAddress the address to serialize
 *
 * @return The FInternetAddr of the given string address. This will point to nullptr on failure.
 */
TSharedPtr<FInternetAddr> FSocketSubsystemSteam::GetAddressFromString(const FString& InAddress)
{
	FString RawAddress = InAddress;
	// Remove steam prefixes if they exist
	RawAddress.RemoveFromStart(STEAM_URL_PREFIX);

	if (RawAddress.IsNumeric())
	{
		// This is a Steam Address
		uint64 Id = FCString::Atoi64(*RawAddress);
		if (Id != 0)
		{
			TSharedRef<FInternetAddrSteam> ReturnAddress = StaticCastSharedRef<FInternetAddrSteam>(CreateInternetAddr());
			ReturnAddress->SetSteamID(CSteamID(Id));
			return ReturnAddress;
		}
		else
		{
			UE_LOGF(LogSockets, Warning, "Could not serialize %ls into a SteamID, the ID was invalid.", *RawAddress);
			return nullptr;
		}
	}
	
	return ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetAddressFromString(InAddress);
}

/**
 * Determines the name of the local machine
 *
 * @param HostName the string that receives the data
 *
 * @return true if successful, false otherwise
 */
bool FSocketSubsystemSteam::GetHostName(FString& HostName)
{
	return false;
}

/**
 *	Create a proper FInternetAddr representation
 */
TSharedRef<FInternetAddr> FSocketSubsystemSteam::CreateInternetAddr()
{
	return MakeShareable(new FInternetAddrSteam());
}

/**
 * @return Whether the machine has a properly configured network device or not
 */
bool FSocketSubsystemSteam::HasNetworkDevice() 
{
	return true;
}

/**
 *	Get the name of the socket subsystem
 * @return a string naming this subsystem
 */
const TCHAR* FSocketSubsystemSteam::GetSocketAPIName() const 
{
	return TEXT("SteamSockets");
}

/**
 * Returns the last error that has happened
 */
ESocketErrors FSocketSubsystemSteam::GetLastErrorCode()
{
	return TranslateErrorCode(LastSocketError);
}

/**
 * Translates the platform error code to a ESocketErrors enum
 */
ESocketErrors FSocketSubsystemSteam::TranslateErrorCode(int32 Code)
{
	// @TODO ONLINE - This needs to be filled in (at present it is 1:1)
	return (ESocketErrors)LastSocketError;
}

bool FSocketSubsystemSteam::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	TArray<TSharedRef<FInternetAddr>> BindArray = GetLocalBindAddresses();
	for (const auto& BindAddr : BindArray)
	{
		OutAddresses.Add(BindAddr);
	}

	return true;
}

/**
 *	Get local IP to bind to
 */
TArray<TSharedRef<FInternetAddr>> FSocketSubsystemSteam::GetLocalBindAddresses()
{
	TArray<TSharedRef<FInternetAddr>> OutArray;

	FInternetAddrSteam* SteamAddr = nullptr;
	CSteamID SteamId;
	if (SteamUser())
	{
		// Prefer the steam user
		SteamId = SteamUser()->GetSteamID();
		SteamAddr = new FInternetAddrSteam(SteamId);
	}
	else if (SteamGameServer() && SteamGameServer()->BLoggedOn())
	{
		// Dedicated server 
		SteamId = SteamGameServer()->GetSteamID();
		SteamAddr = new FInternetAddrSteam(SteamId);
	}
	else
	{
		// Empty/invalid case
		SteamAddr = new FInternetAddrSteam();
	}

	OutArray.Add(MakeShareable(SteamAddr));
	return OutArray;
}

/**
 * Potentially accept an incoming connection from a Steam P2P request
 * 
 * @param SteamNetworkingPtr the interface for access the P2P calls (Client/GameServer)
 * @param RemoteId the id of the incoming request
 * 
 * @return true if accepted, false otherwise
 */
bool FSocketSubsystemSteam::AcceptP2PConnection(ISteamNetworking* SteamNetworkingPtr, const CSteamID& RemoteId)
{
	if (SteamNetworkingPtr && RemoteId.IsValid() && !IsConnectionPendingRemoval(RemoteId, -1))
	{
		UE_LOGF(LogSockets, Log, "Adding P2P connection information with user %llu", RemoteId.ConvertToUint64());
		// Blindly accept connections (but only if P2P enabled)
		SteamNetworkingPtr->AcceptP2PSessionWithUser(RemoteId);
		UE_CLOGF(AcceptedConnections.Contains(RemoteId.ConvertToUint64()), LogSockets, Warning, "User %llu already exists in the connections list!!", RemoteId.ConvertToUint64());
		AcceptedConnections.Add(RemoteId.ConvertToUint64(), FSteamP2PConnectionInfo(SteamNetworkingPtr));
		return true;
	}

	return false;
}

/**
 * Add/update a Steam P2P connection as being recently accessed
 *
 * @param SteamNetworkingPtr proper networking interface that this session is communicating on
 * @param SessionId P2P session recently heard from
 * @param ChannelId the channel id that the update happened on
 *
 * @return true if the connection is active, false if this is in the dead connections list
 */
bool FSocketSubsystemSteam::P2PTouch(ISteamNetworking* SteamNetworkingPtr, const CSteamID& SessionId, int32 ChannelId)
{
    // Don't update any sessions coming from pending disconnects
	if (!IsConnectionPendingRemoval(SessionId, ChannelId))
	{
		FSteamP2PConnectionInfo& ChannelUpdate = AcceptedConnections.FindOrAdd(SessionId.ConvertToUint64());
		ChannelUpdate.SteamNetworkingPtr = SteamNetworkingPtr;

		if (ChannelId != -1)
		{
			ChannelUpdate.AddOrUpdateChannel(ChannelId, FPlatformTime::Seconds());
		}
		return true;
	}

	return false;
}
	
/**
 * Remove a Steam P2P session from tracking and close the connection
 *
 * @param SessionId P2P session to remove
 * @param Channel channel to close, -1 to close all communication
 */
void FSocketSubsystemSteam::P2PRemove(const CSteamID& SessionId, int32 Channel)
{
	FSteamP2PConnectionInfo* ConnectionInfo = AcceptedConnections.Find(SessionId.ConvertToUint64());
	if (ConnectionInfo)
	{
		const bool bRemoveAllConnections = (Channel == -1);
		
		// Only modify the DeadConnections list if we're actively going to change it
		if (!IsConnectionPendingRemoval(SessionId, Channel))
		{
			if (bRemoveAllConnections)
			{
				UE_LOGF(LogSockets, Verbose, "Replacing all existing removals with global removal for %llu", SessionId.ConvertToUint64());
				// Go through and remove all the connections for this user
				for (TMap<FInternetAddrSteam, double>::TIterator It(DeadConnections); It; ++It)
				{
					if (It->Key.GetSteamID() == SessionId)
					{
						It.RemoveCurrent();
					}
				}
			}

			// Move active connections to the dead list so they can be removed (giving Steam a chance to flush connection)
			FInternetAddrSteam RemoveConnection(SessionId);
			RemoveConnection.SetPort(Channel);
			DeadConnections.Add(RemoveConnection, FPlatformTime::Seconds());

			UE_LOGF(LogSockets, Log, "Removing P2P Session Id: %llu, Channel: %d, IdleTime: %0.3f", SessionId.ConvertToUint64(), Channel,
				ConnectionInfo ? (FPlatformTime::Seconds() - ConnectionInfo->LastReceivedTime) : 9999.f);
		}

		if (bRemoveAllConnections)
		{
			// Clean up dead connections will remove the user from the map for us
			UE_CLOGF((ConnectionInfo->ConnectedChannels.Num() > 0), LogSockets, Verbose, "Removing all channel connections for %llu", SessionId.ConvertToUint64());
			ConnectionInfo->ConnectedChannels.Empty();
		}
		else
		{
			bool bWasRemoved = ConnectionInfo->ConnectedChannels.Remove(Channel) > 0;
			UE_CLOGF(bWasRemoved, LogSockets, Verbose, "Removing channel %d from user %llu", Channel, SessionId.ConvertToUint64());
		}
	}
}

/**
 * Checks to see if a Steam P2P Connection is pending close on the given channel.
 *
 * Before checking the given channel, this function checks if the session is marked for
 * global disconnection.
 *
 * @param SessionId the user id tied to the session disconnection
 * @param Channel the communications channel id for the user if it exists
 */
bool FSocketSubsystemSteam::IsConnectionPendingRemoval(const CSteamID& SteamId, int32 Channel)
{
	FInternetAddrSteam RemovalToFind(SteamId);
	RemovalToFind.SetPort(-1);

	// Check with -1 first as that ends all communications with another user
	if (!DeadConnections.Contains(RemovalToFind))
	{
		// If we were asked to check for -1, then early out as we've already checked the entry
		if (Channel == -1)
		{
			return false;
		}

		// Then look for the specific channel instance.
		RemovalToFind.SetPort(Channel);
		return DeadConnections.Contains(RemovalToFind);
	}

	return true;
}

/**
 * Determines if the SocketSubsystemSteam should override the platform
 * socket subsystem. This means ISocketSubsystem::Get() will return this subsystem
 * by default. However, the platform subsystem will still be accessible by
 * specifying ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM) as well as via
 * passthrough operations.
 *
 * If the project does not want to use SteamNetworking features, add
 * bUseSteamNetworking=false to your OnlineSubsystemSteam configuration
 *
 * @return if SteamNetworking should be the default socketsubsystem.
 */
bool FSocketSubsystemSteam::ShouldOverrideDefaultSubsystem() const
{
	bool bOverrideSetting;
	if (GConfig && GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bUseSteamNetworking"), bOverrideSetting, GEngineIni))
	{
		return bOverrideSetting;
	}
	return true;
}

/**
 * Chance for the socket subsystem to get some time
 *
 * @param DeltaTime time since last tick
 */
bool FSocketSubsystemSteam::Tick(float DeltaTime)
{	
    QUICK_SCOPE_CYCLE_COUNTER(STAT_SocketSubsystemSteam_Tick);

	double CurSeconds = FPlatformTime::Seconds();

	// Debug connection state information
	bool bDumpSessionInfo = false;
	if ((CurSeconds - P2PDumpCounter) >= P2PDumpInterval)
	{
		P2PDumpCounter = CurSeconds;
		bDumpSessionInfo = true;
	}

	for (TMap<uint64, FSteamP2PConnectionInfo>::TConstIterator It(AcceptedConnections); It; ++It)
	{
		CSteamID SessionId = CSteamID(It.Key());
		const FSteamP2PConnectionInfo& ConnectionInfo = It.Value();

		bool bExpiredSession = true;
		if (CurSeconds - ConnectionInfo.LastReceivedTime < P2PConnectionTimeout)
		{
			P2PSessionState_t SessionInfo;
			if (ConnectionInfo.SteamNetworkingPtr != nullptr && ConnectionInfo.SteamNetworkingPtr->GetP2PSessionState(SessionId, &SessionInfo))
			{
				bExpiredSession = false;

				if (bDumpSessionInfo)
				{
					UE_LOGF(LogSockets, Verbose, "Dumping Steam P2P socket details:");
					UE_LOGF(LogSockets, Verbose, "- Id: %llu, Number of Channels: %d, IdleTime: %0.3f", SessionId.ConvertToUint64(), ConnectionInfo.ConnectedChannels.Num(), (CurSeconds - ConnectionInfo.LastReceivedTime));

					DumpSteamP2PSessionInfo(SessionInfo);
				}
			}
			else if(ConnectionInfo.ConnectedChannels.Num() > 0) // Suppress this print so that it only prints if we expected to have a connection.
			{
				UE_LOGF(LogSockets, Verbose, "Failed to get Steam P2P session state for Id: %llu, IdleTime: %0.3f", SessionId.ConvertToUint64(), (CurSeconds - ConnectionInfo.LastReceivedTime));
			}
		}

		if (bExpiredSession)
		{
			P2PRemove(SessionId, -1);
		}
	}

	CleanupDeadConnections(false);

	return true;
}

bool FSocketSubsystemSteam::Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("dumpsteamsessions")))
	{
		DumpAllOpenSteamSessions();
		return true;
	}
#endif

	return false;
}

/**
 * Iterate through the pending dead connections and permanently remove any that have been around
 * long enough to flush their contents
 * 
 * @param bSkipLinger skips the timeout reserved for lingering connection data
 */
void FSocketSubsystemSteam::CleanupDeadConnections(bool bSkipLinger)
{
	double CurSeconds = FPlatformTime::Seconds();
	for (TMap<FInternetAddrSteam, double>::TIterator It(DeadConnections); It; ++It)
	{
		const FInternetAddrSteam& SteamConnection = It.Key();
		if (P2PCleanupTimeout == 0.0 || CurSeconds - It.Value() >= P2PCleanupTimeout || bSkipLinger)
		{
			// Only modify connections if the user exists. This check is only done for safety
			if (const FSteamP2PConnectionInfo* ConnectionInfo = AcceptedConnections.Find(SteamConnection.GetSteamID64()))
			{
				bool bShouldRemoveUser = true;
				// All communications are to be removed
				if (SteamConnection.GetPort() == -1)
				{
					UE_LOGF(LogSockets, Log, "Closing all communications with user %ls", *SteamConnection.ToString(false));
					ConnectionInfo->SteamNetworkingPtr->CloseP2PSessionWithUser(SteamConnection.GetSteamID());
				}
				else
				{
					UE_LOGF(LogSockets, Log, "Closing channel %d with user %ls", SteamConnection.GetPort(), *SteamConnection.ToString(false));
					ConnectionInfo->SteamNetworkingPtr->CloseP2PChannelWithUser(SteamConnection.GetSteamID(), SteamConnection.GetPort());
					// If we no longer have any channels open with the user, we must remove the user, as Steam will do this automatically.
					if (ConnectionInfo->ConnectedChannels.Num() != 0)
					{
						bShouldRemoveUser = false;
						UE_LOGF(LogSockets, Verbose, "%ls still has %d open connections.", *SteamConnection.ToString(false), ConnectionInfo->ConnectedChannels.Num());
					}
					else
					{
						UE_LOGF(LogSockets, Verbose, "%ls has no more open connections! Going to remove", *SteamConnection.ToString(false));
					}
				}

				if (bShouldRemoveUser)
				{
					// Remove the user information from our current connections as they are no longer connected to us.
					UE_LOGF(LogSockets, Log, "%ls has been removed.", *SteamConnection.ToString(false));
					AcceptedConnections.Remove(SteamConnection.GetSteamID64());
				}
			}

			It.RemoveCurrent();
		}
	}
}

/**
 * Dumps the Steam P2P networking information for a given session id
 *
 * @param SessionInfo struct from Steam call to GetP2PSessionState
 */
void FSocketSubsystemSteam::DumpSteamP2PSessionInfo(P2PSessionState_t& SessionInfo)
{
	TSharedRef<FInternetAddr> IpAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	IpAddr->SetIp(SessionInfo.m_nRemoteIP);
	IpAddr->SetPort(SessionInfo.m_nRemotePort);
	UE_LOGF(LogSockets, Verbose, "- Detailed P2P session info:");
	UE_LOGF(LogSockets, Verbose, "-- IPAddress: %ls", *IpAddr->ToString(true));
	UE_LOGF(LogSockets, Verbose, "-- ConnectionActive: %i, Connecting: %i, SessionError: %i, UsingRelay: %i",
		SessionInfo.m_bConnectionActive, SessionInfo.m_bConnecting, SessionInfo.m_eP2PSessionError,
		SessionInfo.m_bUsingRelay);
	UE_LOGF(LogSockets, Verbose, "-- QueuedBytes: %i, QueuedPackets: %i", SessionInfo.m_nBytesQueuedForSend,
		SessionInfo.m_nPacketsQueuedForSend);
}

/**
 * Dumps all connection information for each user connection over SteamNet.
 */
void FSocketSubsystemSteam::DumpAllOpenSteamSessions()
{
	UE_LOGF(LogSockets, Verbose, "Current Connection Info: ");
	for (TMap<uint64, FSteamP2PConnectionInfo>::TConstIterator It(AcceptedConnections); It; ++It)
	{
		UE_LOGF(LogSockets, Verbose, "- Connection %llu", It->Key);
		UE_LOGF(LogSockets, Verbose, "--  Last Update Time: %f", It->Value.LastReceivedTime);
		FString ConnectedChannels(TEXT(""));
		for (int32 i = 0; i < It->Value.ConnectedChannels.Num(); ++i)
		{
			ConnectedChannels = FString::Printf(TEXT("%s %d"), *ConnectedChannels, It->Value.ConnectedChannels[i]);
		}
		UE_LOGF(LogSockets, Verbose, "--  Channels:%ls", *ConnectedChannels);
	}
}
