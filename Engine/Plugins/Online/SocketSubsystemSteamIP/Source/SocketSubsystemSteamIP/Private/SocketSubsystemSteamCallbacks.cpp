// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemSteam.h"

// Note: We do not pump the Steam event loop here, we are letting the OSS do that for us.

/**
 *	GameServer API version of connected to Steam backend callback,
 *  initiated by SteamGameServers()->LogOnAnonymous()
 */
void FSocketSubsystemSteam::OnSteamServersConnectedGS(SteamServersConnected_t* CallbackData)
{
	FixupSockets(SteamGameServer()->GetSteamID());
}

/**
 * Notification event from Steam that a P2P connection request has been initiated from a remote connection
 *
 * @param CallbackData information about remote connection request
 */
void FSocketSubsystemSteam::OnP2PSessionRequest(P2PSessionRequest_t* CallbackData)
{
	if (!AcceptP2PConnection(SteamNetworking(), CallbackData->m_steamIDRemote))
	{
		UE_LOGF(LogSockets, Log, "Rejected P2P connection request from remote host");
	}
}

/**
 * Notification event from Steam that a P2P remote connection has failed
 *
 * @param CallbackData information about remote connection failure
 */
void FSocketSubsystemSteam::OnP2PSessionConnectFail(P2PSessionConnectFail_t* CallbackData)
{
	ConnectFailure(CallbackData->m_steamIDRemote);
}

/**
 * Notification event from Steam that a P2P connection request has been initiated from a remote connection
 * (GameServer version)
 *
 * @param CallbackData information about remote connection request
 */
void FSocketSubsystemSteam::OnP2PSessionRequestGS(P2PSessionRequest_t* CallbackData)
{
	if (!AcceptP2PConnection(SteamNetworking(), CallbackData->m_steamIDRemote))
	{
		UE_LOGF(LogSockets, Log, "Rejected P2P connection request from remote host");
	}
}

/**
 * Notification event from Steam that a P2P remote connection has failed
 * (GameServer version)
 *
 * @param CallbackData information about remote connection failure
 */
void FSocketSubsystemSteam::OnP2PSessionConnectFailGS(P2PSessionConnectFail_t* CallbackData)
{
	ConnectFailure(CallbackData->m_steamIDRemote);
}