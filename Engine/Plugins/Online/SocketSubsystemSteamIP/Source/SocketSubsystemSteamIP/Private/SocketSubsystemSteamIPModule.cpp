// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemSteamIPModule.h"
#include "SocketSubsystemSteam.h"
#include "Misc/ConfigCacheIni.h"
#include "SteamSharedModule.h"
#include "SocketSubsystemModule.h"

IMPLEMENT_MODULE(FSocketSubsystemSteamIPModule, SocketSubsystemSteamIP);


void FSocketSubsystemSteamIPModule::StartupModule()
{
	FSteamSharedModule& SharedModule = FSteamSharedModule::Get();
	const bool bIsNotEditor = (IsRunningDedicatedServer() || IsRunningGame());

	// Load the Steam modules before first call to API
	if (SharedModule.AreSteamDllsLoaded() && bIsNotEditor)
	{
		// Settings flags
		bool bOverrideSocketSubsystem = false;

		// Use this flag from the SteamNetworking configuration so that the IPNetDrivers can be used as needed.
		if (GConfig)
		{
			GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bUseSteamNetworking"), bOverrideSocketSubsystem, GEngineIni);
		}

		// Create and register our singleton factory with the main online subsystem for easy access
		FSocketSubsystemSteam* SocketSubsystem = FSocketSubsystemSteam::Create();
		FString Error;
		if (SocketSubsystem->Init(Error))
		{
			bEnabled = true;

			// Register our socket Subsystem
			FSocketSubsystemModule& SSS = FModuleManager::LoadModuleChecked<FSocketSubsystemModule>("Sockets");
			SSS.RegisterSocketSubsystem(STEAMIP_SUBSYSTEMNAME, SocketSubsystem, bOverrideSocketSubsystem);
		}
		else
		{
			UE_LOGF(LogSockets, Error, "FSocketSubsystemSteamIPModule: Could not initialize SteamSockets, got error: %ls", *Error);
			FSocketSubsystemSteam::Destroy();
		}
	}
	else if (bIsNotEditor)
	{
		UE_LOGF(LogSockets, Warning, "FSocketSubsystemSteamIPModule: Steam SDK %ls libraries not present at %ls or failed to load!", STEAM_SDK_VER, *SharedModule.GetSteamModulePath());
	}
	else
	{
		UE_LOGF(LogSockets, Log, "FSocketSubsystemSteamIPModule: Disabled for editor process.");
	}
}

void FSocketSubsystemSteamIPModule::ShutdownModule()
{
	FModuleManager& ModuleManager = FModuleManager::Get();

	if (ModuleManager.IsModuleLoaded("Sockets"))
	{
		FSocketSubsystemModule& SSS = FModuleManager::GetModuleChecked<FSocketSubsystemModule>("Sockets");
		SSS.UnregisterSocketSubsystem(STEAMIP_SUBSYSTEMNAME);
	}
	FSocketSubsystemSteam::Destroy();
}
