// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketsSteam.h"
#include "SocketSubsystemSteam.h"

bool FSocketSteam::Shutdown(ESocketShutdownMode Mode)
{
	/** Not supported */
	return false;
}

/**
 * Closes the socket
 *
 * @param true if it closes without errors, false otherwise
 */
bool FSocketSteam::Close() 
{
	return true;
}

/**
 * Binds a socket to a network byte ordered address
 *
 * @param Addr the address to bind to
 *
 * @return true if successful, false otherwise
 */
bool FSocketSteam::Bind(const FInternetAddr& Addr) 
{
	SteamChannel = Addr.GetPort();
	return true;
}

/**
 * Connects a socket to a network byte ordered address
 *
 * @param Addr the address to connect to
 *
 * @return true if successful, false otherwise
 */
bool FSocketSteam::Connect(const FInternetAddr& Addr) 
{
	/** Not supported - connectionless (UDP) only */
	return false;
}

/**
 * Places the socket into a state to listen for incoming connections
 *
 * @param MaxBacklog the number of connections to queue before refusing them
 *
 * @return true if successful, false otherwise
 */
bool FSocketSteam::Listen(int32 MaxBacklog) 
{
	/** Not supported - connectionless (UDP) only */
	return false;
}

bool FSocketSteam::WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime)
{
	/** Not supported - connectionless (UDP) only */
	return false;
}

bool FSocketSteam::HasPendingData(uint32& PendingDataSize) 
{
	if (SteamNetworkingPtr->IsP2PPacketAvailable(&PendingDataSize, SteamChannel))
	{
		return (PendingDataSize > 0);
	}

	return false;
}

/**
 * Accepts a connection that is pending
 *
 * @param		SocketDescription debug description of socket
 * @return		The new (heap-allocated) socket, or NULL if unsuccessful.
 */
FSocket* FSocketSteam::Accept(const FString& InSocketDescription) 
{
	/** Not supported - connectionless (UDP) only */
	return nullptr;
}

/**
 * Accepts a connection that is pending
 *
 * @param OutAddr the address of the connection
 * @param		SocketDescription debug description of socket
 *
 * @return		The new (heap-allocated) socket, or NULL if unsuccessful.
 */
FSocket* FSocketSteam::Accept(FInternetAddr& OutAddr, const FString& InSocketDescription) 
{
	/** Not supported - connectionless (UDP) only */
	return nullptr;
}

/**
 * Sends a buffer to a network byte ordered address
 *
 * @param Data the buffer to send
 * @param Count the size of the data to send
 * @param BytesSent out param indicating how much was sent
 * @param Destination the network byte ordered address to send to
 */
bool FSocketSteam::SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination) 
{
	bool bSuccess = false;
	if (SteamNetworkingPtr)
	{
		const FInternetAddrSteam& SteamDest = (const FInternetAddrSteam&)Destination;
		if (SteamDest.GetSteamID() != LocalSteamId)
		{
			if (SteamNetworkingPtr->SendP2PPacket(SteamDest.GetSteamID(), Data, Count, SteamSendMode, SteamDest.GetPort()))
			{
				BytesSent = Count;
				bSuccess = true;
			}
		}		
		else
		{
			UE_LOGF(LogSockets, Warning, "Blocked FSocketSteamworks::SendTo call, directed at localhost");
		}
	}

	return bSuccess;
}

/**
 * Sends a buffer on a connected socket
 *
 * @param Data the buffer to send
 * @param Count the size of the data to send
 * @param BytesSent out param indicating how much was sent
 */
bool FSocketSteam::Send(const uint8* Data, int32 Count, int32& BytesSent) 
{
	/** Not supported - connectionless (UDP) only */
	BytesSent = 0;
	return false;
}

/**
 * Reads a chunk of data from the socket. Gathers the source address too
 *
 * @param Data the buffer to read into
 * @param BufferSize the max size of the buffer
 * @param BytesRead out param indicating how many bytes were read from the socket
 * @param Source out param receiving the address of the sender of the data
 */
bool FSocketSteam::RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags) 
{
	if (Flags != ESocketReceiveFlags::None)
	{
		return false;
	}

	bool bSuccess = true;
	FInternetAddrSteam& SteamAddr = (FInternetAddrSteam&)Source;
	
	uint32 MessageSize = 0;
	CSteamID SteamId;
	if (!SteamNetworkingPtr->ReadP2PPacket(Data, BufferSize, &MessageSize, &SteamId, SteamChannel))
	{
		MessageSize = 0;
        SocketSubsystem->LastSocketError = SE_EWOULDBLOCK;
		bSuccess = false;
	}
	else
	{
		SteamAddr.SetSteamID(SteamId);
		if (SocketSubsystem->P2PTouch(SteamNetworkingPtr, SteamAddr.GetSteamID(), SteamChannel))
		{
			SocketSubsystem->LastSocketError = SE_NO_ERROR;
		}
		else
		{
			MessageSize = 0;
			SocketSubsystem->LastSocketError = SE_UDP_ERR_PORT_UNREACH;
			bSuccess = false;
		}
	}

	// Steam always sends/receives on the same channel both sides
	SteamAddr.SetPort(SteamChannel);
	BytesRead = (int32)MessageSize;

	if (BytesRead > BufferSize)
	{
		UE_LOGF(LogSockets, Error, "FSocketSteam::RecvFrom: Failed to deserialize a packet (length of %d exceeds buffer length of %d), discarding!", BytesRead, BufferSize);
		SocketSubsystem->LastSocketError = SE_EMSGSIZE;
		return false;
	}

	return bSuccess;
}

/**
 * Reads a chunk of data from a connected socket
 *
 * @param Data the buffer to read into
 * @param BufferSize the max size of the buffer
 * @param BytesRead out param indicating how many bytes were read from the socket
 */
bool FSocketSteam::Recv(uint8* Data,int32 BufferSize,int32& BytesRead, ESocketReceiveFlags::Type Flags) 
{
	/** Not supported - connectionless (UDP) only */
	BytesRead = 0;
	return false;
}

bool FSocketSteam::Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime)
{
	// not supported
	return false;
}

/**
 * Determines the connection state of the socket
 */
ESocketConnectionState FSocketSteam::GetConnectionState() 
{
	/** Not supported - connectionless (UDP) only */
	return SCS_NotConnected;
}

/**
 * Reads the address the socket is bound to and returns it
 * 
 * @param OutAddr address the socket is bound to
 */
void FSocketSteam::GetAddress(FInternetAddr& OutAddr) 
{
	FInternetAddrSteam& SteamAddr = (FInternetAddrSteam&)OutAddr;
	SteamAddr.SetSteamID(SteamAddr.GetSteamID());
	SteamAddr.SetPort(SteamAddr.GetPort());
}

bool FSocketSteam::GetPeerAddress(FInternetAddr& OutAddr)
{
	// don't support this	
	return false;
}

/**
 * Sets this socket into non-blocking mode
 *
 * @param bIsNonBlocking whether to enable blocking or not
 *
 * @return true if successful, false otherwise
 */
bool FSocketSteam::SetNonBlocking(bool bIsNonBlocking) 
{
	/** Ignored, not supported */
	return true;
}

/**
 * Sets a socket into broadcast mode (UDP only)
 *
 * @param bAllowBroadcast whether to enable broadcast or not
 *
 * @return true if successful, false otherwise
 */
bool FSocketSteam::SetBroadcast(bool bAllowBroadcast) 
{
	/** Ignored, not supported */
	return true;
}


bool FSocketSteam::JoinMulticastGroup(const FInternetAddr& GroupAddress)
{
	return false;
}


bool FSocketSteam::LeaveMulticastGroup(const FInternetAddr& GroupAddress)
{
	return false;
}


bool FSocketSteam::JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	return false;
}


bool FSocketSteam::LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	return false;
}

bool FSocketSteam::SetMulticastLoopback(bool bLoopback)
{
	return false;
}


bool FSocketSteam::SetMulticastTtl(uint8 TimeToLive)
{
	return false;
}

bool FSocketSteam::SetMulticastInterface(const FInternetAddr& InterfaceAddress)
{
	return false;
}

/**
 * Sets whether a socket can be bound to an address in use
 *
 * @param bAllowReuse whether to allow reuse or not
 *
 * @return true if the call succeeded, false otherwise
 */
bool FSocketSteam::SetReuseAddr(bool bAllowReuse) 
{
	/** Ignored, not supported */
	return true;
}

/**
 * Sets whether and how long a socket will linger after closing
 *
 * @param bShouldLinger whether to have the socket remain open for a time period after closing or not
 * @param Timeout the amount of time to linger before closing
 *
 * @return true if the call succeeded, false otherwise
 */
bool FSocketSteam::SetLinger(bool bShouldLinger, int32 Timeout) 
{
	/** Ignored, not supported */
	return true;
}

/**
 * Enables error queue support for the socket
 *
 * @param bUseErrorQueue whether to enable error queueing or not
 *
 * @return true if the call succeeded, false otherwise
 */
bool FSocketSteam::SetRecvErr(bool bUseErrorQueue) 
{
	/** Ignored, not supported */
	return true;
}

/**
 * Sets the size of the send buffer to use
 *
 * @param Size the size to change it to
 * @param NewSize the out value returning the size that was set (in case OS can't set that)
 *
 * @return true if the call succeeded, false otherwise
 */
bool FSocketSteam::SetSendBufferSize(int32 Size,int32& NewSize) 
{
	/** Ignored, not supported */
	return true;
}

/**
 * Sets the size of the receive buffer to use
 *
 * @param Size the size to change it to
 * @param NewSize the out value returning the size that was set (in case OS can't set that)
 *
 * @return true if the call succeeded, false otherwise
 */
bool FSocketSteam::SetReceiveBufferSize(int32 Size,int32& NewSize) 
{
	/** Ignored, not supported */
	return true;
}

/**
 * Reads the port this socket is bound to.
 */ 
int32 FSocketSteam::GetPortNo() 
{
	return SteamChannel;
}
