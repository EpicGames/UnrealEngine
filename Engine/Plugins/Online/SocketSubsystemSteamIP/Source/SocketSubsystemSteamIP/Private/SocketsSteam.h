// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocketSubsystem.h"
#include "SocketSubsystemSteamTypes.h"
#include "OnlineSubsystemNames.h"
#include "Sockets.h"
#include "SocketSubsystemSteam.h"
#include "SocketSubsystemSteamIPPackage.h"

class FSocketSubsystemSteam;

/**
 * This is the Windows specific socket class
 */
class FSocketSteam :
	public FSocket
{
private:

	/** Reference to the socket subsystem */
	class FSocketSubsystemSteam* SocketSubsystem;

PACKAGE_SCOPE:

	/** Local Steam Id (local network address) */
	CSteamID LocalSteamId;

	/** Channel this socket receives data on (similar to port number) */
	int32 SteamChannel;

	/** Current send mode for SendTo() see EP2PSend in Steam headers */
	EP2PSend SteamSendMode;

	/** Steam P2P interface (depends on client/server)  */
	ISteamNetworking* SteamNetworkingPtr;

	/**
	 * Changes the Steam send mode
	 *
	 * @param NewSendMode send mode to set
	 */
	void SetSteamSendMode(EP2PSend NewSendMode)
	{
		SteamSendMode = NewSendMode;
	}

public:
	/**
	 * Creates a Steam socket
	 *
	 * @param InSocket the socket to assign to this object
	 * @param InSocketType the type of socket that was created
	 * @param InSocketDescription the debug description of the socket
	 * @param InSocketProtocol the protocol used to create this socket.
	 */
	FSocketSteam(ISteamNetworking* InSteamNetworkingPtr, const CSteamID& InLocalSteamId, const FString& InSocketDescription, const FName& InSocketProtocol) :
		FSocket(SOCKTYPE_Datagram, InSocketDescription, InSocketProtocol),
		LocalSteamId(InLocalSteamId),
		SteamChannel(0),
		SteamSendMode(k_EP2PSendUnreliable),
		SteamNetworkingPtr(InSteamNetworkingPtr)
	{
		SocketSubsystem = (FSocketSubsystemSteam*)ISocketSubsystem::Get(STEAMIP_SUBSYSTEMNAME);
	}

	/** Closes the socket if it is still open */
	virtual ~FSocketSteam()
	{
		Close();
	}

	virtual bool Shutdown(ESocketShutdownMode Mode) override;

	/**
	 * Closes the socket
	 *
	 * @param true if it closes without errors, false otherwise
	 */
	virtual bool Close() override final;

	/**
	 * Binds a socket to a network byte ordered address
	 *
	 * @param Addr the address to bind to
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool Bind(const FInternetAddr& Addr) override;

	/**
	 * Connects a socket to a network byte ordered address
	 *
	 * @param Addr the address to connect to
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool Connect(const FInternetAddr& Addr) override;

	/**
	 * Places the socket into a state to listen for incoming connections
	 *
	 * @param MaxBacklog the number of connections to queue before refusing them
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool Listen(int32 MaxBacklog) override;

	virtual bool WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime) override;

	/**
	* Queries the socket to determine if there is pending data on the queue
	*
	* @param PendingDataSize out parameter indicating how much data is on the pipe for a single recv call
	*
	* @return true if the socket has data, false otherwise
	*/
	virtual bool HasPendingData(uint32& PendingDataSize) override;

	/**
	 * Accepts a connection that is pending
	 *
	 * @param		SocketDescription debug description of socket
	 * @return		The new (heap-allocated) socket, or NULL if unsuccessful.
	 */
	virtual class FSocket* Accept(const FString& SocketDescription) override;

	/**
	 * Accepts a connection that is pending
	 *
	 * @param OutAddr the address of the connection
	 * @param		SocketDescription debug description of socket
	 *
	 * @return		The new (heap-allocated) socket, or NULL if unsuccessful.
	 */
	virtual class FSocket* Accept(FInternetAddr& OutAddr, const FString& SocketDescription) override;

	/**
	 * Sends a buffer to a network byte ordered address
	 *
	 * @param Data the buffer to send
	 * @param Count the size of the data to send
	 * @param BytesSent out param indicating how much was sent
	 * @param Destination the network byte ordered address to send to
	 */
	virtual bool SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination) override;

	/**
	 * Sends a buffer on a connected socket
	 *
	 * @param Data the buffer to send
	 * @param Count the size of the data to send
	 * @param BytesSent out param indicating how much was sent
	 */
	virtual bool Send(const uint8* Data, int32 Count, int32& BytesSent) override;

	/**
	 * Reads a chunk of data from the socket. Gathers the source address too
	 *
	 * @param Data the buffer to read into
	 * @param BufferSize the max size of the buffer
	 * @param BytesRead out param indicating how many bytes were read from the socket
	 * @param Source out param receiving the address of the sender of the data
	 * @param Flags the receive flags (must be ESocketReceiveFlags::None)
	 */
	virtual bool RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;

	/**
	 * Reads a chunk of data from a connected socket
	 *
	 * @param Data the buffer to read into
	 * @param BufferSize the max size of the buffer
	 * @param BytesRead out param indicating how many bytes were read from the socket
	 * @param Flags the receive flags
	 */
	virtual bool Recv(uint8* Data,int32 BufferSize,int32& BytesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;

	virtual bool Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime) override;

	/**
	 * Determines the connection state of the socket
	 */
	virtual ESocketConnectionState GetConnectionState() override;

	/**
	 * Reads the address the socket is bound to and returns it
	 * 
	 * @param OutAddr address the socket is bound to
	 */
	virtual void GetAddress(FInternetAddr& OutAddr) override;

	/**
	 * Reads the address of the peer the socket is connected to and returns it
	 * 
	 * @param OutAddr address of the peer the socket is connected to
	 */
	virtual bool GetPeerAddress(FInternetAddr& OutAddr) override;

	/**
	 * Sets this socket into non-blocking mode
	 *
	 * @param bIsNonBlocking whether to enable blocking or not
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool SetNonBlocking(bool bIsNonBlocking = true) override;

	/**
	 * Sets a socket into broadcast mode (UDP only)
	 *
	 * @param bAllowBroadcast whether to enable broadcast or not
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool SetBroadcast(bool bAllowBroadcast = true) override;

	virtual bool SetNoDelay(bool bIsNoDelay = true) override
	{
		return true;
	}

	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) override;

	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress) override;

	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;

	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;

	virtual bool SetMulticastLoopback(bool bLoopback) override;

	virtual bool SetMulticastTtl(uint8 TimeToLive) override;

	virtual bool SetMulticastInterface(const FInternetAddr& InterfaceAddress) override;

	/**
	 * Sets whether a socket can be bound to an address in use
	 *
	 * @param bAllowReuse whether to allow reuse or not
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool SetReuseAddr(bool bAllowReuse = true) override;

	/**
	 * Sets whether and how long a socket will linger after closing
	 *
	 * @param bShouldLinger whether to have the socket remain open for a time period after closing or not
	 * @param Timeout the amount of time to linger before closing
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool SetLinger(bool bShouldLinger = true, int32 Timeout = 0) override;

	/**
	 * Enables error queue support for the socket
	 *
	 * @param bUseErrorQueue whether to enable error queueing or not
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool SetRecvErr(bool bUseErrorQueue = true) override;

	/**
	 * Sets the size of the send buffer to use
	 *
	 * @param Size the size to change it to
	 * @param NewSize the out value returning the size that was set (in case OS can't set that)
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool SetSendBufferSize(int32 Size,int32& NewSize) override;

	/**
	 * Sets the size of the receive buffer to use
	 *
	 * @param Size the size to change it to
	 * @param NewSize the out value returning the size that was set (in case OS can't set that)
	 *
	 * @return true if the call succeeded, false otherwise
	 */
	virtual bool SetReceiveBufferSize(int32 Size,int32& NewSize) override;

	/**
	 * Reads the port this socket is bound to.
	 */ 
	virtual int32 GetPortNo() override;
};
