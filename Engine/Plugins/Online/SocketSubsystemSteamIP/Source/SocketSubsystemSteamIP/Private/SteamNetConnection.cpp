// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamNetConnection.h"
#include "SocketSubsystemSteam.h"
#include "SteamNetDriver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SteamNetConnection)

USteamNetConnection::USteamNetConnection(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	bIsPassthrough(false)
{
}

void USteamNetConnection::InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	bIsPassthrough = InURL.Host.StartsWith(STEAM_URL_PREFIX) ? false : true;
	if (!bIsPassthrough)
	{
		DisableAddressResolution();
	}
	
	Super::InitLocalConnection(InDriver, InSocket, InURL, InState, InMaxPacket, InPacketOverhead);
	if (!bIsPassthrough && RemoteAddr.IsValid())
	{
		FSocketSubsystemSteam* SocketSubsystem = (FSocketSubsystemSteam*)ISocketSubsystem::Get(STEAMIP_SUBSYSTEMNAME);
		if (SocketSubsystem)
		{
			SocketSubsystem->RegisterConnection(this);
		}
	}
}

void USteamNetConnection::InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	bIsPassthrough = ((USteamNetDriver*)InDriver)->bIsPassthrough;
	if (!bIsPassthrough)
	{
		DisableAddressResolution();
	}

	Super::InitRemoteConnection(InDriver, InSocket, InURL, InRemoteAddr, InState, InMaxPacket, InPacketOverhead);
	if (!bIsPassthrough && RemoteAddr.IsValid())
	{
		FSocketSubsystemSteam* SocketSubsystem = (FSocketSubsystemSteam*)ISocketSubsystem::Get(STEAMIP_SUBSYSTEMNAME);
		if (SocketSubsystem)
		{
			SocketSubsystem->RegisterConnection(this);
		}
	}
}

void USteamNetConnection::CleanUp()
{
	Super::CleanUp();

	if (!bIsPassthrough)
	{
		FSocketSubsystemSteam* SocketSubsystem = (FSocketSubsystemSteam*)ISocketSubsystem::Get(STEAMIP_SUBSYSTEMNAME);
		if (SocketSubsystem)
		{
			// Unregister the connection AFTER the parent class has had a chance to close/flush connections
			SocketSubsystem->UnregisterConnection(this);
		}
	}
}
