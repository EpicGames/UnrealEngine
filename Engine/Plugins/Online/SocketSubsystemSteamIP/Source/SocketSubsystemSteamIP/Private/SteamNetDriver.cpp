// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamNetDriver.h"
#include "EngineLogs.h"
#include "SocketsSteam.h"
#include "Misc/CommandLine.h"
#include "SocketSubsystemSteam.h"
#include "OnlineSubsystemNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SteamNetDriver)

USteamNetDriver::USteamNetDriver(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	bIsPassthrough(false)
{
}

void USteamNetDriver::PostInitProperties()
{
	Super::PostInitProperties();
}

bool USteamNetDriver::IsAvailable() const
{
	ISocketSubsystem* SteamSockets = ISocketSubsystem::Get(STEAMIP_SUBSYSTEMNAME);
	if (SteamSockets)
	{
		return true;
	}

	return false;
}

ISocketSubsystem* USteamNetDriver::GetSocketSubsystem()
{
	return ISocketSubsystem::Get(bIsPassthrough ? PLATFORM_SOCKETSUBSYSTEM : STEAMIP_SUBSYSTEMNAME);
}

bool USteamNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	if (bIsPassthrough)
	{
		return UIpNetDriver::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error);
	}

	// Skip UIpNetDriver implementation
	if (!UNetDriver::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		return false;
	}

	ISocketSubsystem* SocketSubsystem = GetSocketSubsystem();
	if (SocketSubsystem == NULL)
	{
		UE_LOGF(LogNet, Warning, "Unable to find socket subsystem");
		Error = TEXT("Unable to find socket subsystem");
		return false;
	}

	if (GetSocket() == nullptr)
	{
		Error = FString::Printf( TEXT("SteamSockets: socket failed (%i)"), (int32)SocketSubsystem->GetLastErrorCode() );
		return false;
	}

	// Bind socket to our port.
	LocalAddr = SocketSubsystem->GetLocalBindAddr(*GLog);

	// Set the Steam channel (port) to communicate on (both Client/Server communicate on same "channel")
	LocalAddr->SetPort(URL.Port);

	int32 AttemptPort = LocalAddr->GetPort();
	int32 BoundPort = SocketSubsystem->BindNextPort(GetSocket(), *LocalAddr, MaxPortCountToTry + 1, 1);
	UE_LOGF(LogNet, Display, "%ls bound to port %d", *GetName(), BoundPort);
	// Success.
	return true;
}

bool USteamNetDriver::InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error)
{
	ISocketSubsystem* SteamSockets = ISocketSubsystem::Get(STEAMIP_SUBSYSTEMNAME);
	if (SteamSockets)
	{
		// If we are opening a Steam URL, create a Steam client socket
		if (ConnectURL.Host.StartsWith(STEAM_URL_PREFIX))
		{
			FUniqueSocket NewSocket = SteamSockets->CreateUniqueSocket(FName(TEXT("SteamClientSocket")), TEXT("Unreal client (Steam)"),
																		FNetworkProtocolTypes::SteamSocketsIP);

			TSharedPtr<FSocket> SharedSocket(NewSocket.Release(), FSocketDeleter(NewSocket.GetDeleter()));

			SetSocketAndLocalAddress(SharedSocket);
		}
		else
		{
			bIsPassthrough = true;
		}
	}

	return Super::InitConnect(InNotify, ConnectURL, Error);
}

bool USteamNetDriver::InitListen(FNetworkNotify* InNotify, FURL& ListenURL, bool bReuseAddressAndPort, FString& Error)
{
	ISocketSubsystem* SteamSockets = ISocketSubsystem::Get(STEAMIP_SUBSYSTEMNAME);
	if (SteamSockets && !ListenURL.HasOption(TEXT("bIsLanMatch")) && !FParse::Param(FCommandLine::Get(), TEXT("forcepassthrough")))
	{
		FName SocketTypeName = IsRunningDedicatedServer() ? FName(TEXT("SteamServerSocket")) : FName(TEXT("SteamClientSocket"));
		FUniqueSocket NewSocket = SteamSockets->CreateUniqueSocket(SocketTypeName, TEXT("Unreal server (Steam)"), FNetworkProtocolTypes::SteamSocketsIP);
		TSharedPtr<FSocket> SharedSocket(NewSocket.Release(), FSocketDeleter(NewSocket.GetDeleter()));

		SetSocketAndLocalAddress(SharedSocket);
	}
	else
	{
        // Socket will be created in base class
		bIsPassthrough = true;
	}

	return Super::InitListen(InNotify, ListenURL, bReuseAddressAndPort, Error);
}

void USteamNetDriver::Shutdown()
{
	if (!bIsPassthrough)
	{
		FSocketSteam* SteamSocket = (FSocketSteam*)GetSocket();
		if (SteamSocket)
		{
			SteamSocket->SetSteamSendMode(k_EP2PSendUnreliableNoDelay);
		}
	}

	Super::Shutdown();
}

bool USteamNetDriver::IsNetResourceValid()
{
	bool bIsValidSteamSocket = !bIsPassthrough && (GetSocket() != nullptr) && ((FSocketSteam*)GetSocket())->LocalSteamId != CSteamID();
	bool bIsValidPassthroughSocket = bIsPassthrough && UIpNetDriver::IsNetResourceValid();
	return bIsValidSteamSocket || bIsValidPassthroughSocket;
}
