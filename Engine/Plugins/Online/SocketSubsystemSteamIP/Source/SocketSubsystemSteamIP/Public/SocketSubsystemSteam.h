// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "Containers/Ticker.h"
#include "SocketSubsystem.h"
#include "SocketSubsystemSteamTypes.h"
#include "IPAddressSteam.h"
#include "SocketSubsystemSteamIPPackage.h"

#define UE_API SOCKETSUBSYSTEMSTEAMIP_API

class FSocketSteam;
class ISteamNetworking;
struct P2PSessionState_t;

class Error;

extern const FName STEAMIP_SUBSYSTEMNAME;

/**
 * Create the socket subsystem for the given platform service
 */
extern FName CreateSteamSocketSubsystem();

/**
 * Tear down the socket subsystem for the given platform service
 */
extern void DestroySteamSocketSubsystem();

/**
 * Windows specific socket subsystem implementation
 */
class FSocketSubsystemSteam : public ISocketSubsystem, public FTSTickerObjectBase, public FSelfRegisteringExec
{
protected:

	/** Single instantiation of this subsystem */
	static UE_API FSocketSubsystemSteam* SocketSingleton;

	/** Tracks existing Steamworks sockets, for connection failure/timeout resolution */
	TArray<class FSocketSteam*> SteamSockets;

	/** Tracks existing Steamworks connections, for connection failure/timeout resolution */
	TArray<struct FWeakObjectPtr> SteamConnections;

	/** Tracks the game server's steam ID given to us by the session interface */
	CSteamID GameServerCSID = CSteamID();

	/** Holds Steam connection information for each user */
	struct FSteamP2PConnectionInfo
	{
		/** Steam networking interface responsible for this connection */
		ISteamNetworking* SteamNetworkingPtr;

		/** 
		 * Last time the user's p2p session had activity (RecvFrom, etc). 
		 * The value of this member is always the max value of the ConnectedChannels object
		 */
		double LastReceivedTime;

		/** 
		 * Channel connection ids for this user
		 */
		TArray<int32> ConnectedChannels;

		FSteamP2PConnectionInfo(ISteamNetworking* InNetworkPtr=nullptr) :
			SteamNetworkingPtr(InNetworkPtr),
			LastReceivedTime(FPlatformTime::Seconds())
		{
		}


		/** This helper function automatically updates LastRecievedTime */
		void AddOrUpdateChannel(int32 InChannel, double InTime)
		{
			ConnectedChannels.AddUnique(InChannel);
			LastReceivedTime = FMath::Max(LastReceivedTime, InTime);
		}
	};

    /** 
	 * List of Steam P2P connections we have
	 * As connections at start do not have a channel id, the key is just the accounts connected to us.
	 */
	TMap<uint64, FSteamP2PConnectionInfo> AcceptedConnections;

	/** 
	 * List of Steam P2P connections to shutdown.
	 * If the FInternetAddrSteam has a channel id of -1, all connections are dropped from the user.
	 * Also tracked is the time in which the connection was marked to be removed (for linger purposes)
	 */
	TMap<FInternetAddrSteam, double> DeadConnections;

	/**
	 * Should Steam P2P sockets all fall back to Steam servers relay if a direct connection fails
	 * read from [OnlineSubsystemSteam.bAllowP2PPacketRelay]
	 */
	bool bAllowP2PPacketRelay;
	/** 
     * Timeout (in seconds) period for any P2P session
	 * read from [OnlineSubsystemSteam.P2PConnectionTimeout]
     * (should be at least as long as NetDriver::ConnectionTimeout) 
     */
	float P2PConnectionTimeout;
	/** Accumulated time before next dump of connection info */
	double P2PDumpCounter;
	/** Connection info output interval */
	double P2PDumpInterval;
	/**
	 * The timeout (in seconds) between when a connection/channel is marked as destroyed
	 * and when it's cleaned up. This allows for catching lingering messages
	 * from other users.
	 * If set to 0, all dead connections will be cleaned up each tick.
	 * read from [OnlineSubsystemSteam.P2PCleanupTimeout]
	 */
	double P2PCleanupTimeout;

	/**
	 * Adds a steam socket for tracking
	 *
	 * @param InSocket	The socket to add for tracking
	 */
	void AddSocket(class FSocketSteam* InSocket)
	{
		SteamSockets.Add(InSocket);
	}

	/**
	 * Removes a steam socket from tracking
	 *
	 * @param InSocket	The socket to remove from tracking
	 */
	void RemoveSocket(class FSocketSteam* InSocket)
	{
		SteamSockets.RemoveSingleSwap(InSocket);
	}

PACKAGE_SCOPE:

	/** 
	 * Singleton interface for this subsystem 
	 * @return the only instance of this subsystem
	 */
	static FSocketSubsystemSteam* Create();

	/**
	 * Performs Windows specific socket clean up
	 */
	static UE_API void Destroy();

	/**
	 * Iterate through the pending dead connections and permanently remove any that have been around
	 * long enough to flush their contents
	 *
	 * @param bSkipLinger skips the timeout reserved for lingering connection information to a race condition
	 */
	UE_API void CleanupDeadConnections(bool bSkipLinger);

	/**
	 * Associate the game server steam id with any sockets that were created prior to successful login
	 *
	 * @param GameServerId id assigned to this game server
	 */
	UE_API void FixupSockets(const CSteamID& GameServerId);

	/**
	 * Adds a steam connection for tracking
	 *
	 * @param Connection The connection to add for tracking
	 */
	UE_API void RegisterConnection(class USteamNetConnection* Connection);

	/**
	 * Removes a steam connection from tracking
	 *
	 * @param Connection The connection to remove from tracking
	 */
	UE_API void UnregisterConnection(class USteamNetConnection* Connection);

	/**
	 * Notification from the Steam event layer that a remote connection has completely failed
	 * 
	 * @param RemoteId failure address
	 */
	UE_API void ConnectFailure(const CSteamID& RemoteId);

	/**
	 * Potentially accept an incoming connection from a Steam P2P request
	 * 
	 * @param SteamNetworkingPtr the interface for access the P2P calls (Client/GameServer)
	 * @param RemoteId the id of the incoming request
	 * 
	 * @return true if accepted, false otherwise
	 */
	UE_API bool AcceptP2PConnection(ISteamNetworking* SteamNetworkingPtr, const CSteamID& RemoteId);

	/**
	 * Add/update a Steam P2P connection as being recently accessed
	 *
	 * @param SteamNetworkingPtr proper networking interface that this session is communicating on
	 * @param SessionId P2P session recently heard from
	 * @param ChannelId the channel id that the update happened on
     *
     * @return true if the connection is active, false if this is in the dead connections list
	 */
	UE_API bool P2PTouch(ISteamNetworking* SteamNetworkingPtr, const CSteamID& SessionId, int32 ChannelId = -1);

	/**
	 * Remove a Steam P2P session from tracking and close the connection
	 *
	 * @param SessionId P2P session to remove
	 * @param Channel channel to close, -1 to close all communication
	 */
	UE_API void P2PRemove(const CSteamID& SessionId, int32 Channel = -1);

	/**
	 * Checks to see if a Steam P2P Connection is pending close on the given channel.
	 *
	 * Before checking the given channel, this function checks if the session is marked for
	 * global disconnection.
	 * 
	 * @param SessionId the user id tied to the session disconnection
	 * @param Channel the communications channel id for the user if it exists
	 */
	UE_API bool IsConnectionPendingRemoval(const CSteamID& SteamId, int32 Channel);

	/**
	 * Determines if the SocketSubsystemSteam should override the platform
	 * socket subsystem. This means ISocketSubsystem::Get() will return this subsystem
	 * by default. However, the platform subsystem will still be accessible by
	 * specifying ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM) as well as via
	 * passthrough operations.
	 *
	 * If the project does not want to use SteamNetworking features, add
	 * bUseSteamNetworking=false to your OnlineSubsystemSteam configuration
	 *
	 * @return if SteamNetworking should be the default socketsubsystem.
	 */
	UE_API bool ShouldOverrideDefaultSubsystem() const;

	/**
	 * Dumps the Steam P2P networking information for a given session state
	 *
	 * @param SessionInfo struct from Steam call to GetP2PSessionState
	 */
	UE_API void DumpSteamP2PSessionInfo(P2PSessionState_t& SessionInfo);

	/**
	 * Dumps all connection information for each user connection over SteamNet.
	 */
	UE_API void DumpAllOpenSteamSessions();



	/** 
	* Delegate registered with Steam to trigger when a server (gameserver API) is connected to the Steam servers.  Initiated with a LogOnAnonymous() call 
	*/
	STEAM_GAMESERVER_CALLBACK(FSocketSubsystemSteam, OnSteamServersConnectedGS, SteamServersConnected_t, OnSteamServersConnectedGSCallback);


	/** Delegate registered with Steam to trigger when another Steam client makes a P2P call to this instance */
	STEAM_CALLBACK(FSocketSubsystemSteam, OnP2PSessionRequest, P2PSessionRequest_t, OnP2PSessionRequestCallback);
	/** Delegate registered with Steam to trigger when a connection between two steam P2P endpoints fails */
	STEAM_CALLBACK(FSocketSubsystemSteam, OnP2PSessionConnectFail, P2PSessionConnectFail_t, OnP2PSessionConnectFailCallback);
	/** Delegate registered with Steam to trigger when another Steam client makes a P2P call to this instance (gameserver API) */
	STEAM_GAMESERVER_CALLBACK(FSocketSubsystemSteam, OnP2PSessionRequestGS, P2PSessionRequest_t, OnP2PSessionRequestGSCallback);
	/** Delegate registered with Steam to trigger when a connection between two steam P2P endpoints fails (gameserver API) */
	STEAM_GAMESERVER_CALLBACK(FSocketSubsystemSteam, OnP2PSessionConnectFailGS, P2PSessionConnectFail_t, OnP2PSessionConnectFailGSCallback);

public:

	/**
	 * If the session interface wishes to bring up a game server socket, it must call this first when the policy response has been received to update us with the corresponding SteamId to use in the socket layer
	 */
	void UpdateGameServerId(const CSteamID& InGameServerCSID)
	{
		GameServerCSID = InGameServerCSID;
	}

	/** Last error set by the socket subsystem or one of its sockets */
	int32 LastSocketError;

public:

	FSocketSubsystemSteam() :
	    bAllowP2PPacketRelay(false),
		P2PConnectionTimeout(45.0f),
		P2PDumpCounter(0.0),
		P2PDumpInterval(10.0),
		P2PCleanupTimeout(1.5),
		OnSteamServersConnectedGSCallback(this, &FSocketSubsystemSteam::OnSteamServersConnectedGS),
		OnP2PSessionRequestCallback(this, &FSocketSubsystemSteam::OnP2PSessionRequest),
		OnP2PSessionConnectFailCallback(this, &FSocketSubsystemSteam::OnP2PSessionConnectFail),
		OnP2PSessionRequestGSCallback(this, &FSocketSubsystemSteam::OnP2PSessionRequestGS),
		OnP2PSessionConnectFailGSCallback(this, &FSocketSubsystemSteam::OnP2PSessionConnectFailGS),
		LastSocketError(0)
	{
	}

	/**
	 * Does Windows platform initialization of the sockets library
	 *
	 * @param Error a string that is filled with error information
	 *
	 * @return true if initialized ok, false otherwise
	 */
	UE_API virtual bool Init(FString& Error) override;

	/**
	 * Performs platform specific socket clean up
	 */
	UE_API virtual void Shutdown() override;

	/**
	 * Creates a socket
	 *
	 * @Param SocketType type of socket to create (DGram, Stream, etc)
	 * @param SocketDescription debug description
	 * @param ProtocolType the socket protocol to be used
	 *
	 * @return the new socket or NULL if failed
	 */

	UE_API virtual class FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;

	/**
	 * Cleans up a socket class
	 *
	 * @param Socket the socket object to destroy
	 */
	UE_API virtual void DestroySocket(class FSocket* Socket) override;

	/**
	 * Gets the address information of the given hostname and outputs it into an array of resolvable addresses.
	 * It is up to the caller to determine which one is valid for their environment.
	 *
	 * @param HostName string version of the queryable hostname or ip address
	 * @param ServiceName string version of a service name ("http") or a port number ("80")
	 * @param QueryFlags What flags are used in making the getaddrinfo call. Several flags can be used at once by ORing the values together.
	 *                   Platforms are required to translate this value into a the correct flag representation.
	 * @param ProtocolType this is used to limit results from the call. Specifying None will search all valid protocols.
	 *					   Callers will find they rarely have to specify this flag.
	 * @param SocketType What socket type should the results be formatted for. This typically does not change any formatting results and can
	 *                   be safely left to the default value.
	 *
	 * @return the array of results from GetAddrInfo
	 */
	UE_API virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr,
		EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default,
		const FName ProtocolTypeName = NAME_None,
		ESocketType SocketType = ESocketType::SOCKTYPE_Unknown) override;


	/**
	 * Serializes a string that only contains an address.
	 * 
	 * On Steam, this will take SteamIDs and serialize them into FInternetAddrSteam if it is determined
	 * the input string is an ID. Otherwise, this will give you back a FInternetAddrBSD.
	 *
	 * This is a what you see is what you get, there is no DNS resolution of the input string, 
	 * so only use this if you know you already have a valid ip address. 
	 * Otherwise, feed the address to GetAddressInfo for guaranteed results.
	 *
	 * @param IPAddress the ip address to serialize
	 *
	 * @return The FInternetAddr of the given string address. This will point to nullptr on failure.
	 */
	UE_API virtual TSharedPtr<FInternetAddr> GetAddressFromString(const FString& IPAddress) override;

	/**
	 * Some platforms require chat data (voice, text, etc.) to be placed into
	 * packets in a special way. This function tells the net connection
	 * whether this is required for this platform
	 */
	virtual bool RequiresChatDataBeSeparate() override
	{
		return false;
	}

	/**
	 * Some platforms require packets be encrypted. This function tells the
	 * net connection whether this is required for this platform
	 */
	virtual bool RequiresEncryptedPackets() override
	{
		return false;
	}

	/**
	 * Determines the name of the local machine
	 *
	 * @param HostName the string that receives the data
	 *
	 * @return true if successful, false otherwise
	 */
	UE_API virtual bool GetHostName(FString& HostName) override;

	/**
	 *	Create a proper FInternetAddr representation
	 */
	UE_API virtual TSharedRef<FInternetAddr> CreateInternetAddr() override;

	/**
	 * @return Whether the machine has a properly configured network device or not
	 */
	UE_API virtual bool HasNetworkDevice() override;

	/**
	 *	Get the name of the socket subsystem
	 * @return a string naming this subsystem
	 */
	UE_API virtual const TCHAR* GetSocketAPIName() const override;

	/**
	 * Returns the last error that has happened
	 */
	UE_API virtual ESocketErrors GetLastErrorCode() override;

	/**
	 * Translates the platform error code to a ESocketErrors enum
	 */
	UE_API virtual ESocketErrors TranslateErrorCode(int32 Code) override;

	/**
	 * Gets the list of addresses associated with the adapters on the local computer.
	 *
	 * @param OutAdresses - Will hold the address list.
	 *
	 * @return true on success, false otherwise.
	 */
	UE_API virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses) override;

	/**
	 *	Get local IP to bind to
	 */
	UE_API virtual TArray<TSharedRef<FInternetAddr>> GetLocalBindAddresses() override;

	/**
	 * Chance for the socket subsystem to get some time
	 *
	 * @param DeltaTime time since last tick
	 */
	UE_API virtual bool Tick(float DeltaTime) override;

	/**
	 * Waiting on a socket is not supported.
	 */
	virtual bool IsSocketWaitSupported() const override { return false; }

protected:
	// FSelfRegisteringExec
	UE_API virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
};

#undef UE_API
