// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

#define UE_API SOCKETSUBSYSTEMSTEAMIP_API

class FSocketSubsystemSteamIPModule : public IModuleInterface
{
public:

	FSocketSubsystemSteamIPModule() : bEnabled(false)
	{

	}

	virtual ~FSocketSubsystemSteamIPModule()
	{
	}

	// IModuleInterface

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	static inline class FSocketSubsystemSteamIPModule& Get()
	{
		return FModuleManager::LoadModuleChecked<class FSocketSubsystemSteamIPModule>("SocketSubsystemSteamIP");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("SocketSubsystemSteamIP");
	}

	bool IsEnabled() const { return bEnabled; }

private:
	bool bEnabled;
};

#undef UE_API
