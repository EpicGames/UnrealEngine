// Copyright Epic Games, Inc. All Rights Reserved.

#undef PACKAGE_SCOPE
#ifdef SOCKETSUBSYSTEMSTEAMIP_MODULE
#define PACKAGE_SCOPE public
#else
#define PACKAGE_SCOPE protected
#endif
