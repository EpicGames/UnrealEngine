// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HAL/Platform.h"

THIRD_PARTY_INCLUDES_START
#include <steam/isteamuser.h>
#include <steam/isteamnetworking.h>
THIRD_PARTY_INCLUDES_END