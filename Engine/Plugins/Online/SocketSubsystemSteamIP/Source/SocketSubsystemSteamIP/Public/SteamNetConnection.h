// Copyright Epic Games, Inc. All Rights Reserved.

//
// Steam sockets based implementation of a network connection used by the Steam net driver class
//

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "IpConnection.h"
#include "SteamNetConnection.generated.h"

UCLASS(transient, config=Engine)
class USteamNetConnection : public UIpConnection
{
    GENERATED_UCLASS_BODY()

	//~ Begin UIpConnection Interface
	virtual void InitRemoteConnection(class UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	virtual void InitLocalConnection(class UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	virtual void CleanUp() override;
	//~ End UIpConnection Interface

	/** Is this net connection passthrough to IpConnection */
	UPROPERTY()
	bool bIsPassthrough;

	/** Access for monitoring / cleanup */
	friend class FSocketSubsystemSteam;
};
