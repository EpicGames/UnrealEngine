// Copyright Epic Games, Inc. All Rights Reserved.

//
// Steam sockets based implementation of the net driver
//

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "IpNetDriver.h"
#include "SteamNetDriver.generated.h"

class Error;
class FNetworkNotify;

UCLASS(transient, config=Engine)
class USteamNetDriver : public UIpNetDriver
{
	GENERATED_UCLASS_BODY()

	/** Should this net driver behave as a passthrough to normal IP */
	bool bIsPassthrough;

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UIpNetDriver Interface

	virtual class ISocketSubsystem* GetSocketSubsystem() override;
	virtual bool IsAvailable() const override;
	virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	virtual bool InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error) override;	
	virtual bool InitListen(FNetworkNotify* InNotify, FURL& ListenURL, bool bReuseAddressAndPort, FString& Error) override;
	virtual void Shutdown() override;
	virtual bool IsNetResourceValid() override;

	//~ End UIpNetDriver Interface

};
