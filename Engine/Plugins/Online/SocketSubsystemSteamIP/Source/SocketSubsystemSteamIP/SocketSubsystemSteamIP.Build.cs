// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

public class SocketSubsystemSteamIP : ModuleRules
{
	public SocketSubsystemSteamIP(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDefinitions.Add("SOCKETSUBSYSTEMSTEAMIP_MODULE=1");
		Type = ModuleType.CPlusPlus;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"NetCore",
				"Sockets",
				"OnlineSubsystemUtils"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreOnline",
				"SteamShared",
				"CoreUObject"
			}
		);
		AddEngineThirdPartyPrivateStaticDependencies(Target, "Steamworks");
	}
}
